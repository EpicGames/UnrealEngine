// Copyright Epic Games, Inc. All Rights Reserved.

#include "Models/ScreenComparisonModel.h"
#include "ISourceControlModule.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "SourceControlHelpers.h"
#include "ISourceControlProvider.h"
#include "Misc/Paths.h"

FScreenComparisonModel::FScreenComparisonModel(const FComparisonReport& InReport)
	: Report(InReport)
	, bComplete(false)
{
	const FImageComparisonResult& ComparisonResult = Report.GetComparisonResult();

	/*
		Remember that this report may have been loaded from elsewhere so we will not have access to the comparison/delta paths that
		were used at the time. We need to use the files in the report folder, though we will write to the ideal approved path since
		that is where a blessed image should be checked in to.
	*/

	FString SourceImage = FPaths::Combine(Report.GetReportRootDirectory(), Report.GetComparisonResult().ReportIncomingFilePath);
	FString OutputImage = FPaths::Combine(FPaths::ProjectDir(), Report.GetComparisonResult().IdealApprovedFolderPath, FPaths::GetCleanFilename(Report.GetComparisonResult().IncomingFilePath));

	FileImports.Add(FFileMapping(OutputImage, SourceImage));
	FileImports.Add(FFileMapping(FPaths::ChangeExtension(OutputImage, TEXT("json")), FPaths::ChangeExtension(SourceImage, TEXT("json"))));

	TryLoadMetadata();
}

bool FScreenComparisonModel::IsComplete() const
{
	return bComplete;
}

void FScreenComparisonModel::Complete(bool bWasSuccessful)
{
	if (bWasSuccessful)
	{
		// Delete report folder
		IFileManager::Get().DeleteDirectory(*Report.GetReportPath(), false, true);
	}

	bComplete = true;
	OnComplete.Broadcast();
}

void FScreenComparisonModel::TryLoadMetadata()
{
	if (!Metadata.IsSet())
	{
		const FImageComparisonResult& Comparison = Report.GetComparisonResult();

		FString IncomingImage = Report.GetReportRootDirectory() / Comparison.ReportIncomingFilePath;
		FString IncomingMetadata = FPaths::ChangeExtension(IncomingImage, TEXT("json"));

		if (!IncomingMetadata.IsEmpty())
		{
			FString Json;
			if (FFileHelper::LoadFileToString(Json, *IncomingMetadata))
			{
				FAutomationScreenshotMetadata LoadedMetadata;
				if (FJsonObjectConverter::JsonObjectStringToUStruct(Json, &LoadedMetadata, 0, 0))
				{
					Metadata = LoadedMetadata;
				}
			}
		}
	}
}

TOptional<FAutomationScreenshotMetadata> FScreenComparisonModel::GetMetadata()
{
	// Load it just in case.
	TryLoadMetadata();

	return Metadata;
}

FString FScreenComparisonModel::GetName()
{
	if (Name.IsEmpty())
	{
		auto LoadedMetadata = GetMetadata();
		if (LoadedMetadata.IsSet())
		{
			FString VariantSuffix = LoadedMetadata->VariantName.Len() > 0 ? FString::Printf(TEXT(".%s"), *LoadedMetadata->VariantName) : TEXT("");
			FString NameString = FString::Printf(TEXT("%s.%s%s"), *LoadedMetadata->Context, *LoadedMetadata->ScreenShotName, *VariantSuffix);

			Name = NameString;
		}
	}

	return Name;
}

bool FScreenComparisonModel::AddNew()
{
	bool bSuccess = true;

	// Copy the files from the reports location to the destination location
	TArray<FString> SourceControlFiles;
	for ( const FFileMapping& Incoming : FileImports)
	{
		if (IFileManager::Get().Copy(*Incoming.DestinationFile, *Incoming.SourceFile, true, true) != 0)
		{
			bSuccess = false;
		}

		SourceControlFiles.Add(Incoming.DestinationFile);
	}

	if (bSuccess)
	{
		// Add the files to source control
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		if (SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), SourceControlFiles) == ECommandResult::Failed)
		{
			// TODO Error
		}
	}

	Complete(bSuccess);

	return bSuccess;
}

bool FScreenComparisonModel::Replace()
{
	// @todo(agrant): test this

	// Delete all the existing files in this area
	RemoveExistingApproved();

	// Copy files to the approved
	const FString ImportIncomingRoot = Report.GetReportPath();

	TArray<FString> SourceControlFiles;
	for ( const FFileMapping& Incoming : FileImports)
	{
		SourceControlFiles.Add(Incoming.DestinationFile);
	}

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	if (!USourceControlHelpers::RevertFiles(SourceControlFiles))
	{
		//TODO Error
	}

	SourceControlFiles.Reset();

	// Copy the files from the reports location to the destination location
	for (const FFileMapping& Incoming : FileImports)
	{
		IFileManager::Get().Copy(*Incoming.DestinationFile, *Incoming.SourceFile, true, true);
		SourceControlFiles.Add(Incoming.DestinationFile);
	}

	if (!USourceControlHelpers::CheckOutOrAddFiles(SourceControlFiles))
	{
		//TODO Error
	}

	Complete(true);

	return true;
}

bool FScreenComparisonModel::RemoveExistingApproved()
{
	FString ApprovedFolder = FPaths::Combine(FPaths::ProjectDir(), Report.GetComparisonResult().ApprovedFilePath);

	TArray<FString> SourceControlFiles;

	IFileManager::Get().FindFilesRecursive(SourceControlFiles, *FPaths::GetPath(ApprovedFolder), TEXT("*.*"), true, false, false);

	if (SourceControlFiles.Num())
	{
		return USourceControlHelpers::MarkFilesForDelete(SourceControlFiles);
	}

	return true;
}

bool FScreenComparisonModel::AddAlternative()
{
	// @todo(agrant): test this

	// Copy files to the approved
	const FString ImportIncomingRoot = Report.GetReportPath();

	TArray<FString> SourceControlFiles;
	for ( const FFileMapping& Import : FileImports )
	{
		SourceControlFiles.Add(Import.DestinationFile);
	}

	if (!USourceControlHelpers::RevertFiles(SourceControlFiles))
	{
		//TODO Error
	}

	for ( const FFileMapping& Import : FileImports )
	{
		if ( IFileManager::Get().Copy(*Import.DestinationFile, *Import.SourceFile, false, true) == COPY_OK )
		{
			SourceControlFiles.Add(Import.DestinationFile);
		}
		else
		{
			// TODO Error
		}
	}

	if (!USourceControlHelpers::CheckOutOrAddFiles(SourceControlFiles))
	{
		//TODO Error
	}

	Complete(true);

	return true;
}
