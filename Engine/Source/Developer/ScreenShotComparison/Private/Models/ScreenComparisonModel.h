// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "ImageComparer.h"
#include "AutomationWorkerMessages.h"
#include "Misc/FileHelper.h"
#include "JsonObjectConverter.h"
#include "AutomationWorkerMessages.h"

class FScreenComparisonModel
{
public:
	FScreenComparisonModel(const FComparisonReport& InReport);

	DECLARE_MULTICAST_DELEGATE(FOnComplete);
	FOnComplete OnComplete;

	FComparisonReport Report;

	bool IsComplete() const;

	void Complete(bool WasSuccessful);

	bool AddNew();
	bool Replace();
	bool AddAlternative();

	TOptional<FAutomationScreenshotMetadata> GetMetadata();
	FString GetName();

private:

	bool RemoveExistingApproved();
	void TryLoadMetadata();

private:
	bool bComplete;

	TOptional<FAutomationScreenshotMetadata> Metadata;
	FString Name;

	struct FFileMapping
	{
		FFileMapping(const FString& InDestFile, const FString& InSourceFile)
			: DestinationFile(InDestFile)
			, SourceFile(InSourceFile)
		{
		}

		// local file we'd write to on disk
		FString DestinationFile;

		// input file from the report
		FString SourceFile;		
	};

	// 
	TArray<FFileMapping> FileImports;
};
