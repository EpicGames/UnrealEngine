// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScreenShotComparisonModule.cpp: Implements the FScreenShotComparisonModule class.
=============================================================================*/

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Interfaces/IScreenShotComparisonModule.h"
#include "Widgets/SScreenShotBrowser.h"

static const FName ScreenShotComparisonName("ScreenShotComparison");


/**
 * Implements the ScreenShotComparison module.
 */
class FScreenShotComparisonModule
	: public IScreenShotComparisonModule
{
public:
	virtual TSharedRef<class SWidget> CreateScreenShotComparison( const IScreenShotManagerRef& ScreenShotManager ) override
	{
		return SNew(SScreenShotBrowser, ScreenShotManager);
	}

};


IMPLEMENT_MODULE(FScreenShotComparisonModule, ScreenShotComparison);
