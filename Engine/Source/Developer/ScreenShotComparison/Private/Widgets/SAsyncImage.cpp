// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SAsyncImage.h"

#include "HAL/FileManager.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SOverlay.h"


void SAsyncImage::Construct(const FArguments& InArgs)
{
	bLoaded = false;

	FModuleManager::Get().LoadModuleChecked(FName("ImageWrapper"));

	ImageFilePath = InArgs._ImageFilePath;

	ChildSlot
	[
		SNew(SOverlay)

		+ SOverlay::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SAssignNew(Progress, SCircularThrobber)
		]

		+ SOverlay::Slot()
		[
			SAssignNew(Image, SImage)
			.Visibility(EVisibility::Collapsed)
		]
	];

	// Enqueue the request to load the screenshot on the thread pool.
	FString ImagePath = InArgs._ImageFilePath;
	TWeakPtr<SAsyncImage> TempWeakThis = SharedThis(this);
	TextureFuture = Async(EAsyncExecution::ThreadPool, [TempWeakThis, ImagePath] () {

		if ( TempWeakThis.IsValid() )
		{
			return SAsyncImage::LoadScreenshot(ImagePath);
		}

		return ( FSlateTextureDataPtr )nullptr;
	});
}

void SAsyncImage::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if ( !bLoaded && TextureFuture.IsReady() )
	{
		bLoaded = true;

		if ( IFileManager::Get().FileExists(*ImageFilePath) )
		{
			FSlateTextureDataPtr TextureData = TextureFuture.Get();
			if ( FSlateApplication::Get().GetRenderer()->GenerateDynamicImageResource(*ImageFilePath, TextureData.ToSharedRef()) )
			{
				DynamicBrush = MakeShareable(new FSlateDynamicImageBrush(*ImageFilePath, FVector2D(TextureData->GetWidth(), TextureData->GetHeight())));
				Image->SetImage(DynamicBrush.Get());
				Image->SetVisibility(EVisibility::SelfHitTestInvisible);

				Progress->SetVisibility(EVisibility::Collapsed);
			}
		}
		else
		{
			Image->SetImage(nullptr);
			Image->SetVisibility(EVisibility::SelfHitTestInvisible);
			Progress->SetVisibility(EVisibility::Collapsed);
		}
	}
}

TSharedPtr<FSlateDynamicImageBrush> SAsyncImage::GetDynamicBrush()
{
	return DynamicBrush;
}

FSlateTextureDataPtr SAsyncImage::LoadScreenshot(FString ImagePath)
{
	TArray<uint8> RawFileData;
	if ( FFileHelper::LoadFileToArray(RawFileData, *ImagePath) )
	{
		IImageWrapperModule& ImageWrapperModule = FModuleManager::GetModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
		TSharedPtr<IImageWrapper> ImageWrappers[3] =
		{
			ImageWrapperModule.CreateImageWrapper(EImageFormat::PNG),
			ImageWrapperModule.CreateImageWrapper(EImageFormat::JPEG),
			ImageWrapperModule.CreateImageWrapper(EImageFormat::BMP),
		};

		for ( TSharedPtr<IImageWrapper> ImageWrapper : ImageWrappers )
		{
			if ( ImageWrapper.IsValid() && ImageWrapper->SetCompressed(RawFileData.GetData(), RawFileData.Num()) )
			{
				TArray<uint8> RawData;
				if ( ImageWrapper->GetRaw(ERGBFormat::BGRA, 8, RawData) )
				{
					return MakeShareable(new FSlateTextureData(ImageWrapper->GetWidth(), ImageWrapper->GetHeight(), 4, RawData));
				}
			}
		}
	}

	return nullptr;
}
