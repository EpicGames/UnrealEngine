// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SThrobber.h"
#include "Textures/SlateTextureData.h"
#include "Async/Async.h"

class SAsyncImage : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAsyncImage) {}
		SLATE_ARGUMENT(FString, ImageFilePath)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	TSharedPtr<FSlateDynamicImageBrush> GetDynamicBrush();

private:
	FString ImageFilePath;
	TSharedPtr<SImage> Image;
	TSharedPtr<SCircularThrobber> Progress;
	bool bLoaded;
	TFuture<FSlateTextureDataPtr> TextureFuture;
	TSharedPtr<FSlateDynamicImageBrush> DynamicBrush;

	static FSlateTextureDataPtr LoadScreenshot(FString ImagePath);
};