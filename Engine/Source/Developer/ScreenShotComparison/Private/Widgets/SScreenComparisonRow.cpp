// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SScreenComparisonRow.h"
#include "Models/ScreenComparisonModel.h"
#include "Modules/ModuleManager.h"
#include "ISourceControlModule.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlProvider.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Input/SCheckBox.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "IImageWrapperModule.h"
#include "Framework/Application/SlateApplication.h"
#include "JsonObjectConverter.h"
#include "Widgets/SScreenShotImagePopup.h"
#include "Widgets/SAsyncImage.h"

#define LOCTEXT_NAMESPACE "SScreenShotBrowser"

class SImageComparison : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SImageComparison) {}
		SLATE_ARGUMENT(TSharedPtr<FSlateDynamicImageBrush>, BaseImage)
		SLATE_ARGUMENT(TSharedPtr<FSlateDynamicImageBrush>, ModifiedImage)
		SLATE_ARGUMENT(TSharedPtr<FSlateDynamicImageBrush>, DeltaImage)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		BaseImage = InArgs._BaseImage;
		ModifiedImage = InArgs._ModifiedImage;
		DeltaImage = InArgs._DeltaImage;

		ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SScaleBox)
				.Stretch(EStretch::ScaleToFit)
				[
					SNew(SOverlay)

					+ SOverlay::Slot()
					[
						SNew(SImage)
						.Image(BaseImage.Get())
					]
						
					+ SOverlay::Slot()
					[
						SNew(SImage)
						.Image(ModifiedImage.Get())
						.ColorAndOpacity(this, &SImageComparison::GetModifiedOpacity)
					]

					+ SOverlay::Slot()
					[
						SNew(SImage)
						.Image(DeltaImage.Get())
						.ColorAndOpacity(this, &SImageComparison::GetDeltaOpacity)
					]
				]
			]
			
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			[
				SNew(SHorizontalBox)
				
				// Comparison slider
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("GroundTruth", "Ground Truth"))
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.MinDesiredWidth(150)
					[
						SAssignNew(OpacitySlider, SSlider)
						.Value(0.5f)
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Incoming", "Incoming"))
				]

				// Delta checkbox
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.MinDesiredWidth(50)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					[
						SAssignNew(DeltaCheckbox, SCheckBox)
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ShowDelta", "Show Delta"))
				]
			]
		];
	}

	FSlateColor GetModifiedOpacity() const
	{
		return FLinearColor(1, 1, 1, OpacitySlider->GetValue());
	}

	FSlateColor GetDeltaOpacity() const
	{
		return FLinearColor(1, 1, 1, DeltaCheckbox->IsChecked() ? 1 : 0);
	}

private:
	TSharedPtr<FSlateDynamicImageBrush> BaseImage;
	TSharedPtr<FSlateDynamicImageBrush> ModifiedImage;
	TSharedPtr<FSlateDynamicImageBrush> DeltaImage;

	TSharedPtr<SSlider> OpacitySlider;
	TSharedPtr<SCheckBox> DeltaCheckbox;
};

class SNewImage : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNewImage) {}
		SLATE_ARGUMENT(TSharedPtr<FSlateDynamicImageBrush>, NewBaseImage)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		NewBaseImage = InArgs._NewBaseImage;

		ChildSlot
			[
				SNew(SScaleBox)
					.Stretch(EStretch::ScaleToFit)
					[
						SNew(SImage)
							.Image(NewBaseImage.Get())
					]
			];
	}


private:
	TSharedPtr<FSlateDynamicImageBrush> NewBaseImage;
};

void SScreenComparisonRow::Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
{
	ScreenshotManager = InArgs._ScreenshotManager;
	ComparisonDirectory = InArgs._ComparisonDirectory;
	Model = InArgs._ComparisonResult;

	CachedActualImageSize = FIntPoint::NoneValue;

	SMultiColumnTableRow<TSharedPtr<FScreenComparisonModel>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}

TSharedRef<SWidget> SScreenComparisonRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if ( ColumnName == "Name" )
	{
		auto ModelMetaData = Model->GetMetadata();

		if (ModelMetaData.IsSet())
		{
			const FImageComparisonResult& ComparisonResult = Model->Report.GetComparisonResult();

			FSlateColor TextColor = FSlateColor::UseForeground();

			if (ComparisonResult.IsNew())
			{
				TextColor = FSlateColor(FLinearColor::Yellow);
			}
			else if (!ComparisonResult.AreSimilar())
			{
				TextColor = FSlateColor(FLinearColor(FColor::Orange));
			}

			return SNew(STextBlock)
				.Text(GetName())
				.ColorAndOpacity(TextColor);
		}
		else
		{
			return SNew(STextBlock).Text(GetName());
		}
	}
	else if (ColumnName == "Date")
	{
		const FImageComparisonResult& ComparisonResult = Model->Report.GetComparisonResult();
		const FDateTime& CreationTime = ComparisonResult.CreationTime;		
		FString Entry = FString::Printf(TEXT("%04d/%02d/%02d - %02d:%02d"), 
			CreationTime.GetYear(),CreationTime.GetMonth(), CreationTime.GetDay(),
			CreationTime.GetHour(), CreationTime.GetMinute());
		return SNew(STextBlock).Text(FText::FromString(Entry));
	}
	else if (ColumnName == "Platform")
	{
		const FImageComparisonResult& ComparisonResult = Model->Report.GetComparisonResult();
		FString Entry = FString::Printf(TEXT("%s %s"), *ComparisonResult.SourcePlatform, *ComparisonResult.SourceRHI);
		return SNew(STextBlock).Text(FText::FromString(Entry));
	}
	else if ( ColumnName == "Delta" )
	{
		FNumberFormattingOptions Format;
		Format.MinimumFractionalDigits = 2;
		Format.MaximumFractionalDigits = 2;

		const FImageComparisonResult& Comparison = Model->Report.GetComparisonResult();

		const FText GlobalDelta = FText::AsPercent(Comparison.GlobalDifference, &Format);
		const FText LocalDelta = FText::AsPercent(Comparison.MaxLocalDifference, &Format);

		const FText Differences = FText::Format(LOCTEXT("LocalvGlobalDelta", "{0} | {1}"), LocalDelta, GlobalDelta);
		return SNew(STextBlock).Text(Differences);
	}
	else if ( ColumnName == "Preview" )
	{
		const FImageComparisonResult& ComparisonResult = Model->Report.GetComparisonResult();
		if ( ComparisonResult.IsNew() )
		{
			return BuildAddedView();
		}
		else if (IsComparingAgainstPlatformFallback())
		{
			return
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					BuildComparisonPreview()
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.IsEnabled(this, &SScreenComparisonRow::CanAddPlatformSpecificNew)
						.Text(LOCTEXT("AddPlatformSpecificNew", "Add Platform-Specific New"))
						.OnClicked(this, &SScreenComparisonRow::AddPlatformSpecificNew)
					]
				];
		}
		else
		{
			return
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					BuildComparisonPreview()
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.IsEnabled(this, &SScreenComparisonRow::CanReplace)
						.Text(LOCTEXT("Replace", "Replace"))
						.OnClicked(this, &SScreenComparisonRow::Replace)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(10, 0, 0, 0)
					[
						SNew(SButton)
						.IsEnabled(this, &SScreenComparisonRow::CanAddAsAlternative)
						.Text(LOCTEXT("AddAlternative", "Add As Alternative"))
						.OnClicked(this, &SScreenComparisonRow::AddAlternative)
					]

					+ SHorizontalBox::Slot()
					.Padding(10, 0, 0, 0)
					.AutoWidth()
					[
						SNew(SButton)
						.IsEnabled(true)
						.Text(LOCTEXT("Delete", "Delete"))
						.OnClicked(this, &SScreenComparisonRow::Remove)
					]
				];
		}
	}

	return SNullWidget::NullWidget;
}

bool SScreenComparisonRow::CanUseSourceControl() const
{
	return ISourceControlModule::Get().IsEnabled();
}

FText SScreenComparisonRow::GetAddNewButtonTooltip() const
{
	if (ISourceControlModule::Get().IsEnabled())
	{
		return LOCTEXT("AddNewToolTip", "Add new ground truth image to revision control.");
	}
	else
	{
		return LOCTEXT("AddNewToolTip_Disabled", "Cannot add new ground truth image. Please connect to revision control.");
	}
}

bool SScreenComparisonRow::IsComparingAgainstPlatformFallback() const
{
	const FImageComparisonResult& Comparison = Model->Report.GetComparisonResult();
	bool bHasApprovedFile = !Comparison.ApprovedFilePath.IsEmpty();	
	return bHasApprovedFile && !Comparison.IsIdeal();
}

TSharedRef<SWidget> SScreenComparisonRow::BuildAddedView()
{
	const FImageComparisonResult& ComparisonResult = Model->Report.GetComparisonResult();
	FString IncomingFile = FPaths::Combine(Model->Report.GetReportRootDirectory(), ComparisonResult.ReportIncomingFilePath);

	return
		SNew(SVerticalBox)
		
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBox)
			.HeightOverride(100)
			.HAlign(HAlign_Left)
			[
				SNew(SScaleBox)
				.Stretch(EStretch::ScaleToFit)
				[
					SNew(SHorizontalBox)
		
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(4.0f, 4.0f)
					[
						SNew(SBorder)
						.BorderImage(nullptr)
						.OnMouseButtonDown(this, &SScreenComparisonRow::OnCompareNewImage)
						[
							SAssignNew(UnapprovedImageWidget, SAsyncImage)
							.ImageFilePath(IncomingFile)
							.ToolTipText(FText::FromString(IncomingFile))
						]
						
					]
				]
			]
		]
		
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Center)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.IsEnabled(this, &SScreenComparisonRow::CanUseSourceControl)
				.Text(LOCTEXT("AddNew", "Add New!"))
				.ToolTipText(this, &SScreenComparisonRow::GetAddNewButtonTooltip)
				.OnClicked(this, &SScreenComparisonRow::AddNew)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10, 0, 0, 0)
			[
				SNew(SButton)
				.IsEnabled(true)
				.Text(LOCTEXT("Delete", "Delete"))
				.OnClicked(this, &SScreenComparisonRow::Remove)
			]
		];
}

TSharedRef<SWidget> SScreenComparisonRow::BuildComparisonPreview()
{
	const FImageComparisonResult& ComparisonResult = Model->Report.GetComparisonResult();

	FString ApprovedFile = FPaths::Combine(Model->Report.GetReportRootDirectory(), ComparisonResult.ReportApprovedFilePath);

	// If the actual approved file is on disk then use that so the tool-tip is more useful
	if (IFileManager::Get().FileExists(*ComparisonResult.ApprovedFilePath))
	{
		ApprovedFile = ComparisonResult.ApprovedFilePath;
	}

	FString IncomingFile = FPaths::Combine(Model->Report.GetReportRootDirectory(), ComparisonResult.ReportIncomingFilePath);
	FString DeltaFile = FPaths::Combine(Model->Report.GetReportRootDirectory(), ComparisonResult.ReportComparisonFilePath);

	// Create the screen shot data widget.
	return 
		SNew(SBorder)
		.BorderImage(nullptr)
		.OnMouseButtonDown(this, &SScreenComparisonRow::OnCompareImages)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBox)
				.HeightOverride(100)
				[
					SNew(SScaleBox)
					.Stretch(EStretch::ScaleToFit)
					[
						SNew(SHorizontalBox)
					
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(4.0f, 4.0f)
						[
							SAssignNew(ApprovedImageWidget, SAsyncImage)
							.ImageFilePath(ApprovedFile)
							.ToolTipText(FText::FromString(ApprovedFile))
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(4.0f, 4.0f)
						[
							SAssignNew(DeltaImageWidget, SAsyncImage)
							.ImageFilePath(DeltaFile)
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(4.0f, 4.0f)
						[
							SAssignNew(UnapprovedImageWidget, SAsyncImage)
							.ImageFilePath(IncomingFile)
							.ToolTipText(FText::FromString(IncomingFile))
						]
					]
				]
			]
	
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT( "GroundTruth", "Ground Truth" ))
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Difference", "Difference"))
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Incoming", "Incoming"))
				]
			]
		];
}

bool SScreenComparisonRow::CanAddNew() const
{
	return CanUseSourceControl();
}

FReply SScreenComparisonRow::AddNew()
{
	Model->AddNew();

	return FReply::Handled();
}

bool SScreenComparisonRow::CanAddPlatformSpecificNew() const
{
	return CanUseSourceControl() && IsComparingAgainstPlatformFallback();
}

FReply SScreenComparisonRow::AddPlatformSpecificNew()
{
	Model->AddNew();

	return FReply::Handled();
}

bool SScreenComparisonRow::CanReplace() const
{
	return CanUseSourceControl() && !IsComparingAgainstPlatformFallback();
}

FReply SScreenComparisonRow::Replace()
{
	Model->Replace();
	return FReply::Handled();
}

bool SScreenComparisonRow::CanAddAsAlternative() const
{
	const FImageComparisonResult& Comparison = Model->Report.GetComparisonResult();
	return CanUseSourceControl()
		&& !Comparison.AreSimilar()
		&& (Comparison.IncomingFilePath != Comparison.ApprovedFilePath) 
		&& !IsComparingAgainstPlatformFallback();
}

FReply SScreenComparisonRow::AddAlternative()
{
	Model->AddAlternative();
	return FReply::Handled();
}

FReply SScreenComparisonRow::Remove()
{
	Model->Complete(true);

	return FReply::Handled();
}

FReply SScreenComparisonRow::OnCompareImages(const FGeometry& InGeometry, const FPointerEvent& InEvent)
{
	TSharedPtr<FSlateDynamicImageBrush> ApprovedImage = ApprovedImageWidget->GetDynamicBrush();
	TSharedPtr<FSlateDynamicImageBrush> UnapprovedImage = UnapprovedImageWidget->GetDynamicBrush();
	TSharedPtr<FSlateDynamicImageBrush> DeltaImage = DeltaImageWidget->GetDynamicBrush();

	if ( ApprovedImage.IsValid() && UnapprovedImage.IsValid() && DeltaImage.IsValid()  )
	{
		TSharedRef<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared()).ToSharedRef();

		// Center ourselves in the parent window
		TSharedRef<SWindow> PopupWindow = SNew(SWindow)
			.IsPopupWindow(false)
			.ClientSize(FVector2D(1280, 720))
			.SizingRule(ESizingRule::UserSized)
			.AutoCenter(EAutoCenter::PreferredWorkArea)
			.SupportsMaximize(true)
			.SupportsMinimize(true)
			.FocusWhenFirstShown(true)
			.ActivationPolicy(EWindowActivationPolicy::Always)
			.Content()
			[
				SNew(SImageComparison)
				.BaseImage(ApprovedImage)
				.ModifiedImage(UnapprovedImage)
				.DeltaImage(DeltaImage)
			];

		FSlateApplication::Get().AddWindowAsNativeChild(PopupWindow, ParentWindow, true);
	}

	return FReply::Handled();
}

FReply SScreenComparisonRow::OnCompareNewImage(const FGeometry& InGeometry, const FPointerEvent& InEvent)
{

	const FImageComparisonResult& ComparisonResult = Model->Report.GetComparisonResult();
	FString IncomingFilePath = FPaths::Combine(Model->Report.GetReportRootDirectory(), ComparisonResult.ReportIncomingFilePath);

	TSharedPtr<FSlateDynamicImageBrush> UnapprovedImage = UnapprovedImageWidget->GetDynamicBrush();

	if (UnapprovedImage.IsValid())
	{
		TSharedRef<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared()).ToSharedRef();

		TSharedRef<SWindow> PopupWindow = SNew(SWindow)
			.IsPopupWindow(false)
			.ClientSize(FVector2D(1280,720))
			.SizingRule(ESizingRule::UserSized)
			.AutoCenter(EAutoCenter::PreferredWorkArea)
			.SupportsMaximize(true)
			.SupportsMinimize(true)
			.FocusWhenFirstShown(true)
			.ActivationPolicy(EWindowActivationPolicy::Always)
			.Content()
			[
				SNew(SNewImage)
				.NewBaseImage(UnapprovedImage)
			];

		FSlateApplication::Get().AddWindowAsNativeChild(PopupWindow, ParentWindow, true);
	}
	
	return FReply::Handled();
}

FReply SScreenComparisonRow::OnImageClicked(const FGeometry& InGeometry, const FPointerEvent& InEvent, TSharedPtr<FSlateDynamicImageBrush> Image)
{
	TSharedRef<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared()).ToSharedRef();

	// Center ourselves in the parent window
	TSharedRef<SWindow> PopupWindow = SNew(SWindow)
		.IsPopupWindow(false)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(Image->ImageSize)
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMaximize(true)
		.SupportsMinimize(true)
		.FocusWhenFirstShown(true)
		.ActivationPolicy(EWindowActivationPolicy::Always)
		.Content()
		[
			SNew(SScreenShotImagePopup)
			.ImageBrush(Image)
			.ImageSize(Image->ImageSize.IntPoint())
		];

	FSlateApplication::Get().AddWindowAsNativeChild(PopupWindow, ParentWindow, true);

	return FReply::Handled();
}

FText SScreenComparisonRow::GetName() const
{
	if (!Name.IsSet())
	{
		FString ModelName = Model->GetName();
		if (!ModelName.IsEmpty())
		{
			Name = FText::FromString(ModelName);
		}
		else
		{
			Name = LOCTEXT("Unknown", "Unknown Test, no metadata discovered.");
		}
	}

	return Name.GetValue();
}

#undef LOCTEXT_NAMESPACE
