// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "ISourceControlState.h"
#include "Interfaces/IScreenShotManager.h"
#include "Async/Async.h"

class FScreenComparisonModel;
struct FSlateDynamicImageBrush;
class SAsyncImage;

/**
 * Widget to display a particular view.
 */
class SScreenComparisonRow : public SMultiColumnTableRow< TSharedPtr<FScreenComparisonModel> >
{
public:

	SLATE_BEGIN_ARGS( SScreenComparisonRow )
		{}

		SLATE_ARGUMENT( IScreenShotManagerPtr, ScreenshotManager )
		SLATE_ARGUMENT( FString, ComparisonDirectory )
		SLATE_ARGUMENT( TSharedPtr<FScreenComparisonModel>, ComparisonResult )

	SLATE_END_ARGS()
	
	/**
	 * Construct the widget.
	 *
	 * @param InArgs   A declaration from which to construct the widget.
 	 * @param InOwnerTableView   The owning table data.
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView );
	
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	
	/**
	 * Get content of the Name column.
	 *
	 * @return FText object corresponding to the current object's content of the Name column.
	 */
	FText GetName() const;

private:
	bool CanUseSourceControl() const;
	bool IsComparingAgainstPlatformFallback() const;

	TSharedRef<SWidget> BuildAddedView();
	TSharedRef<SWidget> BuildComparisonPreview();

	FText GetAddNewButtonTooltip() const;
	bool CanAddNew() const;
	FReply AddNew();

	bool CanAddPlatformSpecificNew() const;
	FReply AddPlatformSpecificNew();

	bool CanReplace() const;
	FReply Replace();

	bool CanAddAsAlternative() const;
	FReply AddAlternative();

	FReply Remove();

	FReply OnCompareImages(const FGeometry& InGeometry, const FPointerEvent& InEvent);
	FReply OnCompareNewImage(const FGeometry& InGeometry, const FPointerEvent& InEvent);
	FReply OnImageClicked(const FGeometry& InGeometry, const FPointerEvent& InEvent, TSharedPtr<FSlateDynamicImageBrush> Image);

private:

	//Holds the screen shot info.
	TSharedPtr<FScreenComparisonModel> Model;

	mutable TOptional<FText> Name;

	// The manager containing the screen shots
	IScreenShotManagerPtr ScreenshotManager;

	FString ComparisonDirectory;

	//The cached actual size of the screenshot
	FIntPoint CachedActualImageSize;

	TSharedPtr<SAsyncImage> ApprovedImageWidget;
	TSharedPtr<SAsyncImage> DeltaImageWidget;
	TSharedPtr<SAsyncImage> UnapprovedImageWidget;
};
