// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SScreenShotBrowser.cpp: Implements the SScreenShotBrowser class.
=============================================================================*/

#include "Widgets/SScreenShotBrowser.h"
#include "HAL/FileManager.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SDirectoryPicker.h"
#include "Widgets/SScreenComparisonRow.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Misc/FeedbackContext.h"
#include "Styling/AppStyle.h"
#include "Modules/ModuleManager.h"
#include "DirectoryWatcherModule.h"
#include "IDirectoryWatcher.h"
#include "ISourceControlModule.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlProvider.h"
#include "SPositiveActionButton.h"
#include "SNegativeActionButton.h"

#define LOCTEXT_NAMESPACE "ScreenshotComparison"

void SScreenShotBrowser::Construct( const FArguments& InArgs,  IScreenShotManagerRef InScreenShotManager  )
{
	ScreenShotManager = InScreenShotManager;

	// Default path. This can be set by the UI or will be updated when a new report is generated.
	FString ReportPath = FPaths::AutomationReportsDir();

	// Ensure the report path exists since we will register for change notifications
	if (!IFileManager::Get().DirectoryExists(*ReportPath))
	{
		IFileManager::Get().MakeDirectory(*ReportPath, true);
	}

	ComparisonRoot = FPaths::ConvertRelativePathToFull(ReportPath);
	bReportsChanged = true;
	// Show Errors and New images, but hide Successful images (ie unchanged ones) because fails and new images are the actionable items
	bDisplayingSuccess = false;
	bDisplayingError = true;
	bDisplayingNew = true;
	ReportFilterString = FString();
	
	FModuleManager::Get().LoadModuleChecked(FName("ImageWrapper"));

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f, 0.f)
			[
				SNew(SPositiveActionButton)
				.Text(LOCTEXT("AddAllNewReports", "Add All New Reports"))
				.ToolTipText(LOCTEXT("AddAllNewReportsTooltip", "Adds all filtered new screenshots contained in the reports."))
				.IsEnabled_Lambda([this]() -> bool
					{
						return FilteredComparisonList.Num() > 0 && ISourceControlModule::Get().IsEnabled();
					})
				.OnClicked_Lambda([this]()
					{
						for (int Entry = FilteredComparisonList.Num() - 1; Entry >= 0; --Entry)
						{
							TSharedPtr<FScreenComparisonModel> Model = FilteredComparisonList[Entry];
							const FImageComparisonResult& Comparison = Model->Report.GetComparisonResult();

							if (CanAddNewReportResult(Comparison))
							{
								FilteredComparisonList.RemoveAt(Entry);
								ComparisonList.Remove(Model);
								Model->AddNew();

								// Avoid thrashing P4
								const float SleepBetweenOperations = 0.005f;
								FPlatformProcess::Sleep(SleepBetweenOperations);
							}
						}

						return FReply::Handled();
					})
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f, 0.f)
			[
				SNew(SNegativeActionButton)
				.ActionButtonStyle(EActionButtonStyle::Warning)
				.Text(LOCTEXT("ReplaceAllReports", "Replace All Reports"))
				.ToolTipText(LOCTEXT("ReplaceAllReportsTooltip", "Replaces all filtered screenshots containing a different result in the reports."))
				.IsEnabled_Lambda([this]() -> bool
					{
						return FilteredComparisonList.Num() > 0 && ISourceControlModule::Get().IsEnabled();
					})
				.OnClicked_Lambda([this]()
					{
						for (int Entry = FilteredComparisonList.Num() - 1; Entry >= 0; --Entry)
						{
							TSharedPtr<FScreenComparisonModel> Model = FilteredComparisonList[Entry];
							const FImageComparisonResult& Comparison = Model->Report.GetComparisonResult();

							if (!CanAddNewReportResult(Comparison))
							{
								FilteredComparisonList.RemoveAt(Entry);
								ComparisonList.Remove(Model);
								Model->Replace();

								// Avoid thrashing P4
								const float SleepBetweenOperations = 0.005f;
								FPlatformProcess::Sleep(SleepBetweenOperations);
							}
						}

						return FReply::Handled();
					})
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f, 0.f)
			[
				SNew(SNegativeActionButton)
				.Text(LOCTEXT("DeleteAllReports", "Delete All Reports"))
				.ToolTipText(LOCTEXT("DeleteAllReportsTooltip", "Deletes all the filtered reports. Reports are not removed unless the user resolves them, \nso if you just want to reset the state of the reports, clear them here and then re-run the tests."))
				.IsEnabled_Lambda([this]() -> bool
					{
						return FilteredComparisonList.Num() > 0;
					})
				.OnClicked_Lambda([this]()
					{
						while (FilteredComparisonList.Num() > 0)
						{
							TSharedPtr<FScreenComparisonModel> Model = FilteredComparisonList.Pop();
							Model->Complete(true);
						}

						return FReply::Handled();
					})
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.0f)
			[
				SNew(SDirectoryPicker)
				.Directory(ComparisonRoot)
				.OnDirectoryChanged(this, &SScreenShotBrowser::OnDirectoryChanged)
			]
		]

		+ SVerticalBox::Slot()
			.AutoHeight()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.HAlign(HAlign_Fill)
				.Padding(10.0f, 0.0f)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.Padding(2.0f, 0.0f)
					.HAlign(HAlign_Fill)
					[
						SNew(SSearchBox)
						.HintText(LOCTEXT("ScreenshotFilterHint", "Search"))
						.ToolTipText(LOCTEXT("Search Tests", "Search Tests"))
						.OnTextChanged(this, &SScreenShotBrowser::OnReportFilterTextChanged)
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.0f, 0.0f)
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SCheckBox)
							.HAlign(HAlign_Center)
							.IsChecked(bDisplayingSuccess ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
							.OnCheckStateChanged(this, &SScreenShotBrowser::DisplaySuccess_OnCheckStateChanged)
						]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.AutoWrapText(true)
						.Text(LOCTEXT("DisplaySuccess", "Show Passed"))
						]
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.Padding(2.0f, 0.0f)
					[
						SNew(SHorizontalBox)
					
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SCheckBox)
							.HAlign(HAlign_Center)
							.IsChecked( bDisplayingError ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
							.OnCheckStateChanged(this, &SScreenShotBrowser::DisplayError_OnCheckStateChanged)
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.AutoWrapText(true)
							.Text(LOCTEXT("DisplayErrors", "Show Fails"))
						]
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.0f, 0.0f)
					[
						SNew(SHorizontalBox)
					
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SCheckBox)
							.HAlign(HAlign_Center)
							.IsChecked(bDisplayingNew ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
							.OnCheckStateChanged(this, &SScreenShotBrowser::DisplayNew_OnCheckStateChanged)
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.AutoWrapText(true)
							.Text(LOCTEXT("DisplayNew", "Show New"))
						]
					]
					
					// Empty horizontal slot that works as a spacer
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.Padding(2.0f, 0.0f)
					.HAlign(HAlign_Fill)
				]			
			]
		]
		
		+ SVerticalBox::Slot()
		.FillHeight( 1.0f )
		[
			SAssignNew(ComparisonView, SListView< TSharedPtr<FScreenComparisonModel> >)
			.ListItemsSource(&FilteredComparisonList)
			.OnGenerateRow(this, &SScreenShotBrowser::OnGenerateWidgetForScreenResults)
			.Visibility(this, &SScreenShotBrowser::GetReportsVisibility)
			.SelectionMode(ESelectionMode::None)
			.HeaderRow
			(
				SNew(SHeaderRow)

				+ SHeaderRow::Column("Name")
				.DefaultLabel(LOCTEXT("ColumnHeader_Name", "Name"))
				.FillWidth(1.0f)
				.VAlignCell(VAlign_Center)

				+ SHeaderRow::Column("Date")
				.DefaultLabel(LOCTEXT("ColumnHeader_Date", "Date"))
				.FixedWidth(120)
				.VAlignHeader(VAlign_Center)
				.HAlignHeader(HAlign_Center)
				.HAlignCell(HAlign_Center)
				.VAlignCell(VAlign_Center)

				+ SHeaderRow::Column("Platform")
				.DefaultLabel(LOCTEXT("ColumnHeader_Platform", "Platform"))
				.FixedWidth(120)
				.VAlignHeader(VAlign_Center)
				.HAlignHeader(HAlign_Center)
				.HAlignCell(HAlign_Center)
				.VAlignCell(VAlign_Center)

				+ SHeaderRow::Column("Delta")
				.DefaultLabel(LOCTEXT("ColumnHeader_Delta", "Local | Global Delta"))
				.FixedWidth(120)
				.VAlignHeader(VAlign_Center)
				.HAlignHeader(HAlign_Center)
				.HAlignCell(HAlign_Center)
				.VAlignCell(VAlign_Center)

				+ SHeaderRow::Column("Preview")
				.DefaultLabel(LOCTEXT("ColumnHeader_Preview", "Preview"))
				.FixedWidth(500)
				.HAlignHeader(HAlign_Left)
				.HAlignCell(HAlign_Center)
				.VAlignCell(VAlign_Center)
			)
		]

		+ SVerticalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SThrobber)
			.Visibility(this, &SScreenShotBrowser::GetReportsUpdatingThrobberVisibility)
		]

	];

	RefreshDirectoryWatcher();
}

SScreenShotBrowser::~SScreenShotBrowser()
{
	// Do not load the module if it was already unloaded.
	if (FModuleManager::Get().IsModuleLoaded(TEXT("DirectoryWatcher")))
	{
		FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));

		if (DirectoryWatchingHandle.IsValid())
		{
			DirectoryWatcherModule.Get()->UnregisterDirectoryChangedCallback_Handle(ComparisonRoot, DirectoryWatchingHandle);
		}
	}
}

void SScreenShotBrowser::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if ( bReportsChanged )
	{
		RequestRebuildTree();
	}

	ContinueRebuildTreeIfReady();
	FinishRebuildTreeIfReady();
}

void SScreenShotBrowser::OnDirectoryChanged(const FString& Directory)
{
	ComparisonRoot = Directory;

	RefreshDirectoryWatcher();

	bReportsChanged = true;
}

void SScreenShotBrowser::RefreshDirectoryWatcher()
{
	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
	
	if ( DirectoryWatchingHandle.IsValid() )
	{
		DirectoryWatcherModule.Get()->UnregisterDirectoryChangedCallback_Handle(ComparisonRoot, DirectoryWatchingHandle);
	}

	DirectoryWatcherModule.Get()->RegisterDirectoryChangedCallback_Handle(ComparisonRoot, IDirectoryWatcher::FDirectoryChanged::CreateRaw(this, &SScreenShotBrowser::OnReportsChanged), DirectoryWatchingHandle, IDirectoryWatcher::WatchOptions::IncludeDirectoryChanges);
}

void SScreenShotBrowser::OnReportsChanged(const TArray<struct FFileChangeData>& /*FileChanges*/)
{
	bReportsChanged = true;
}

TSharedRef<ITableRow> SScreenShotBrowser::OnGenerateWidgetForScreenResults(TSharedPtr<FScreenComparisonModel> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	check(InItem.IsValid());

	TSharedRef<SScreenComparisonRow> ResultWidget = SNew(SScreenComparisonRow, OwnerTable)
		.ScreenshotManager(ScreenShotManager)
		.ComparisonDirectory(ComparisonRoot)
		.ComparisonResult(InItem);

	return ResultWidget;
}

void SScreenShotBrowser::DisplaySuccess_OnCheckStateChanged(ECheckBoxState NewRadioState)
{
	bDisplayingSuccess = (NewRadioState == ECheckBoxState::Checked);
	ApplyReportFilterToWidgets();
}

void SScreenShotBrowser::DisplayError_OnCheckStateChanged(ECheckBoxState NewRadioState)
{
	bDisplayingError = (NewRadioState == ECheckBoxState::Checked);
	ApplyReportFilterToWidgets();
}

void SScreenShotBrowser::DisplayNew_OnCheckStateChanged(ECheckBoxState NewRadioState)
{
	bDisplayingNew = (NewRadioState == ECheckBoxState::Checked);
	ApplyReportFilterToWidgets();
}

void SScreenShotBrowser::OnReportFilterTextChanged(const FText& InText)
{
	ReportFilterString = InText.ToString();
	ApplyReportFilterToWidgets();
}

bool SScreenShotBrowser::MatchesReportFilterCriteria(const FString& ComparisonName, const FImageComparisonResult& ComparisonResult) const
{
	if (!ReportFilterString.IsEmpty() && !ComparisonName.Contains(ReportFilterString, ESearchCase::IgnoreCase))
	{
		return false;
	}

	bool bIsNew = ComparisonResult.IsNew();
	bool bIsFail = !bIsNew && !ComparisonResult.AreSimilar();
	bool bIsPass = !bIsNew && !bIsFail;

	if (bIsPass && !bDisplayingSuccess)
	{
		return false;
	}
	if (bIsNew && !bDisplayingNew)
	{
		return false;
	}
	if (bIsFail && !bDisplayingError)
	{
		return false;
	}

	return true;
}

void SScreenShotBrowser::ApplyReportFilterToWidgets()
{
	FilteredComparisonList.Reset();
	for (auto& Item : ComparisonList)
	{
		if(Item.IsValid() && MatchesReportFilterCriteria(Item->GetName(), Item->Report.GetComparisonResult()))
		{
			FilteredComparisonList.Add(Item);
		}
	}
	ComparisonView->RequestListRefresh();
}

void SScreenShotBrowser::RequestRebuildTree()
{
	bReportsChanged = false;

	PendingOpenComparisonReportsResult.Reset();
	PendingScreenComparisonModelsLoadResult.Reset();
	ComparisonList.Empty();
	FilteredComparisonList.Reset();
	CurrentReports.Reset();

	PendingOpenComparisonReportsResult = ScreenShotManager->OpenComparisonReportsAsync(ComparisonRoot);
}

void SScreenShotBrowser::ContinueRebuildTreeIfReady()
{
	check(IsInGameThread());

	if (PendingOpenComparisonReportsResult.IsValid()
		&& PendingOpenComparisonReportsResult.IsReady())
	{
		CurrentReports = PendingOpenComparisonReportsResult.Get();
		PendingOpenComparisonReportsResult.Reset();

		if ((!CurrentReports.IsValid()) || CurrentReports->IsEmpty())
		{
			return;
		}
		//Sort by what will resolve down to the Screenshots' names
		CurrentReports->Sort([](const FComparisonReport& LHS, const FComparisonReport& RHS) { return LHS.GetReportPath().Compare(RHS.GetReportPath()) < 0; });

		PendingScreenComparisonModelsLoadResult = Async(EAsyncExecution::ThreadPool, [CurrentReportsWPtr = CurrentReports.ToWeakPtr()]() -> TArray<TSharedPtr<FScreenComparisonModel>>
		{
			int32 CurrentReportsNum = 0;
			{
				TSharedPtr<TArray<FComparisonReport>> CurrentReportsSPtr = CurrentReportsWPtr.Pin();
				if (CurrentReportsSPtr.IsValid())
				{
					CurrentReportsNum = CurrentReportsSPtr->Num();
				}
				else
				{
					// The current reports list is outdated. Cancel the job.
					return {};
				}
			}


			if (CurrentReportsNum <= 0)
			{
				return {};
			}

			TArray<TSharedPtr<FScreenComparisonModel>> Result;
			Result.Reserve(CurrentReportsNum);

			for (int32 CurrentReportIndex = 0; CurrentReportIndex < CurrentReportsNum; ++CurrentReportIndex)
			{
				TSharedPtr<TArray<FComparisonReport>> CurrentReportsSPtr = CurrentReportsWPtr.Pin();
				if (CurrentReportsSPtr.IsValid())
				{
					const FComparisonReport& Report = (*CurrentReportsSPtr)[CurrentReportIndex];
					Result.Add(MakeShared<FScreenComparisonModel>(Report));
				}
				else
				{
					// The current reports list is outdated. Cancel the job.
					return {};
				}
			}

			return Result;
		});
	}
}

void SScreenShotBrowser::FinishRebuildTreeIfReady()
{
	check(IsInGameThread());

	if (PendingScreenComparisonModelsLoadResult.IsValid()
		&& PendingScreenComparisonModelsLoadResult.IsReady())
	{
		ComparisonList = PendingScreenComparisonModelsLoadResult.Get();
		PendingScreenComparisonModelsLoadResult.Reset();

		for (auto& Model : ComparisonList)
		{
			Model->OnComplete.AddLambda([this, Model]() {
				ComparisonList.Remove(Model);
				FilteredComparisonList.Remove(Model);
				ComparisonView->RequestListRefresh();
			});
		}

		ApplyReportFilterToWidgets();
	}
}

EVisibility SScreenShotBrowser::GetReportsVisibility() const
{
	return (PendingOpenComparisonReportsResult.IsValid() || PendingScreenComparisonModelsLoadResult.IsValid() ? EVisibility::Collapsed : EVisibility::Visible);
}

EVisibility SScreenShotBrowser::GetReportsUpdatingThrobberVisibility() const
{
	return (GetReportsVisibility() == EVisibility::Visible ? EVisibility::Collapsed : EVisibility::Visible);
}

bool SScreenShotBrowser::CanAddNewReportResult(const FImageComparisonResult& Comparison)
{
	return Comparison.IsNew();
}

#undef LOCTEXT_NAMESPACE
