// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SScreenShotBrowser.h: Declares the SScreenShotBrowser class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Interfaces/IScreenShotManager.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Images/SThrobber.h"
#include "Models/ScreenComparisonModel.h"

/**
 * Implements a Slate widget for browsing active game sessions.
 */

class SScreenShotBrowser : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SScreenShotBrowser) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs - The declaration data for this widget.
 	 * @param InScreenShotManager - The screen shot manager containing the screen shot data.
	 */
	void Construct( const FArguments& InArgs, IScreenShotManagerRef InScreenShotManager );
	virtual ~SScreenShotBrowser();

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

public:

	TSharedRef<ITableRow> OnGenerateWidgetForScreenResults(TSharedPtr<FScreenComparisonModel> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	void DisplaySuccess_OnCheckStateChanged(ECheckBoxState NewRadioState);
	void DisplayError_OnCheckStateChanged(ECheckBoxState NewRadioState);
	void DisplayNew_OnCheckStateChanged(ECheckBoxState NewRadioState);
	void OnReportFilterTextChanged(const FText& InText);

private:

	void OnDirectoryChanged(const FString& Directory);

	void RefreshDirectoryWatcher();

	void OnReportsChanged(const TArray<struct FFileChangeData>& /*FileChanges*/);

	/**
	 * Requests regeneration of the widgets when it is needed.
	 */
	void RequestRebuildTree();

	/**
	* Rebuilds screenshot tree if there are any pending comparison reports that are ready to be displayed.
	*/
	void ContinueRebuildTreeIfReady();

	/**
	* Finishes rebuilding of screenshot tree if all the report model metadata is loaded.
	*/
	void FinishRebuildTreeIfReady();

	/**
	* Gets reports tree widget visibility.
	*/
	EVisibility GetReportsVisibility() const;

	/**
	* Gets throbber visibility (report tree rebuild process is in progress).
	*/
	EVisibility GetReportsUpdatingThrobberVisibility() const;

	bool CanAddNewReportResult(const FImageComparisonResult& Comparison);

	/**
	* Checks whether or not the information is valid against the report filtering criteria.
	*
	* @param ComparisonName - The name of the item to be checked against.
	* @param ComparisonResult - The ImageComparisonResult used to be checked against the display criteria.
	* @return true if the passed in information matches the current filtering criteria
	*/
	bool MatchesReportFilterCriteria(const FString& ComparisonName, const FImageComparisonResult& ComparisonResult) const;

	/**
	* Apply the current report filter to the widgets of comparison view.
	*/
	void ApplyReportFilterToWidgets();

private:

	// The manager containing the screen shots
	IScreenShotManagerPtr ScreenShotManager;

	/** The directory where we're imported comparisons from. */
	FString ComparisonRoot;

	/** The imported screenshot results */
	TSharedPtr<TArray<FComparisonReport>> CurrentReports;

	/** The imported screenshot results copied into an array to be filtered */
	TArray<TSharedPtr<FScreenComparisonModel>> ComparisonList;

	/** The filtered screenshot results copied into an array usable by the list view */
	TArray<TSharedPtr<FScreenComparisonModel>> FilteredComparisonList;

	/**  */
	TSharedPtr< SListView< TSharedPtr<FScreenComparisonModel> > > ComparisonView;

	/** Directory watching handle */
	FDelegateHandle DirectoryWatchingHandle;

	/**  */
	bool bReportsChanged;

	/** Whether or not we're currently displaying successful tests */
	bool bDisplayingSuccess;

	/** Whether or not we're currently displaying tests with errors */
	bool bDisplayingError;

	/** Whether or not we're currently displaying warnings */
	bool bDisplayingNew;

	/** Filter string for our reports, if any */
	FString ReportFilterString;

	/** Stores comparison reports that are loaded asynchronously if ready */
	TFuture<TSharedPtr<TArray<FComparisonReport>>> PendingOpenComparisonReportsResult;

	/** Stores comparison reports' model objects that are loaded asynchronously if ready */
	TFuture<TArray<TSharedPtr<FScreenComparisonModel>>> PendingScreenComparisonModelsLoadResult;
};
