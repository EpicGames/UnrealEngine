// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SScreenShotImagePopup.cpp: Implements the SScreenImagePopup class.
=============================================================================*/

#include "Widgets/SScreenShotImagePopup.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/AppStyle.h"

void SScreenShotImagePopup::Construct( const FArguments& InArgs )
{
	// Load the image
	DynamicImageBrush = InArgs._ImageBrush;

	// Create the screen shot popup widget.
	ChildSlot
	[
		SNew(SBorder)
		. BorderImage(FAppStyle::GetBrush("Menu.Background"))
		. Padding(10)
		[
			SNew(SBox)
			.WidthOverride(InArgs._ImageSize.X)
			.HeightOverride(InArgs._ImageSize.Y)
			[
				SNew(SImage)
				.Image( DynamicImageBrush.Get() )
			]
		]
	];
}
