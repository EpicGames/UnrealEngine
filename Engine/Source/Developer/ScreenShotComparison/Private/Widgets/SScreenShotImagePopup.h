// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SScreenImagePopup.h: Declares the SScreenShotImagePopup class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

/**
 * A widget to show a screen shot image as a pop up window
 */
class SScreenShotImagePopup : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SScreenShotImagePopup ) {}
	
	/** The assest name for the screen shot */
	SLATE_ARGUMENT( TSharedPtr<FSlateDynamicImageBrush>, ImageBrush )

	/** The size of the image */
	SLATE_ARGUMENT( FIntPoint, ImageSize )

	SLATE_END_ARGS()

	/**
	 * Construct the widget.
	 *
	 * @param InArgs   A declaration from which to construct the widget.
 	 * @param InOwnerTableView   The owning table data.
	 */
	void Construct( const FArguments& InArgs );

private:

	// Holds the brush for this screenshot
	TSharedPtr<FSlateDynamicImageBrush> DynamicImageBrush;
};

