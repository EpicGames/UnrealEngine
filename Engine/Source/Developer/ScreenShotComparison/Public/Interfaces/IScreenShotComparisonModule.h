// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IScreenShotComparisonModule.h: Declares the IScreenShotComparisonModule interface.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Interfaces/IScreenShotManager.h"

/**
 * Interface for screen shot comparison UI modules.
 */
class IScreenShotComparisonModule
	: public IModuleInterface
{
public:
	
	/**
	 * Creates a screen shot comparison widget.
	 *
	 * @param InScreenShotManager - the manager containing the screen shot data
	 * @return The new widget
	 */
	virtual TSharedRef<class SWidget> CreateScreenShotComparison( const IScreenShotManagerRef& InScreenShotManager ) = 0;

public:

	/**
	 * Virtual destructor.
	 */
	virtual ~IScreenShotComparisonModule( ) = default;
};
