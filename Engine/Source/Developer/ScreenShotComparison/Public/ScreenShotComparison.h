// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScreenShotComparison.h: ScreenShotComparison module public header file.
=============================================================================*/

#pragma once


/* Boilerplate
 *****************************************************************************/

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()


/* Dependencies
 *****************************************************************************/

#include "Core.h"
#include "Modules/ModuleInterface.h"
#include "SlateBasics.h"
#include "Styling/AppStyle.h"
#include "ScreenShotComparisonTools.h"


/* Interfaces
 *****************************************************************************/
#include "Interfaces/IScreenShotComparisonModule.h"
