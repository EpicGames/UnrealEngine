// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ScreenShotComparison : ModuleRules
{
	public ScreenShotComparison(ReadOnlyTargetRules Target) : base(Target)
	{
		IWYUSupport = IWYUSupport.None;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ToolWidgets",
				"InputCore",
				"ScreenShotComparisonTools",
				"Slate",
				"SlateCore",
				"ImageWrapper",
				"CoreUObject",
				"DesktopWidgets",
				"SourceControl",
				"AutomationMessages",
				"Json",
				"JsonUtilities",
				"DirectoryWatcher"
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
			}
		);
	}
}
